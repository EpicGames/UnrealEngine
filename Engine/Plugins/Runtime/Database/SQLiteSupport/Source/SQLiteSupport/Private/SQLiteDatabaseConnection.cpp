// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQLiteDatabaseConnection.h"
#include "SQLiteResultSet.h"

bool FSQLiteDatabaseConnection::Execute(const TCHAR* CommandString, FSQLiteResultSet*& RecordSet)
{
	RecordSet = nullptr;

	if (Database.IsValid())
	{
		// Compile the statement/query
		FSQLitePreparedStatement PreparedStatement = Database.PrepareStatement(CommandString);
		if (PreparedStatement.IsValid())
		{
			// Initialize records from compiled query
			RecordSet = new FSQLiteResultSet(MoveTemp(PreparedStatement));
			return true;
		}
	}

	return false;
}

bool FSQLiteDatabaseConnection::Execute(const TCHAR* CommandString)
{
	return Database.IsValid() && Database.Execute(CommandString);
}

bool FSQLiteDatabaseConnection::Execute(const TCHAR* CommandString, FDataBaseRecordSet*& RecordSet)
{
	return Execute(CommandString, (FSQLiteResultSet*&)RecordSet);
}

void FSQLiteDatabaseConnection::Close()
{
	Database.Close();
}

bool FSQLiteDatabaseConnection::Open(const TCHAR* ConnectionString, const TCHAR* RemoteConnectionIP, const TCHAR* RemoteConnectionStringOverride)
{
	return Database.Open(ConnectionString);
}

FString FSQLiteDatabaseConnection::GetLastError()
{
	return Database.GetLastError();
}
