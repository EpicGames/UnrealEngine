// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQLiteResultSet.h"
#include "CoreGlobals.h"
#include "SQLiteDatabase.h"
#include "SQLiteTypes.h"

FSQLiteResultSet::FSQLiteResultSet(FSQLitePreparedStatement&& InPreparedStatement)
	: PreparedStatement(MoveTemp(InPreparedStatement))
{
	check(PreparedStatement.IsValid());

	StepResult = PreparedStatement.Step(&StepResultErrorCode);
	if (StepResult == ESQLitePreparedStatementStepResult::Row)
	{
		const TArray<FString>& ColumnNames = PreparedStatement.GetColumnNames();
		for (int32 ColumnIndex = 0; ColumnIndex < ColumnNames.Num(); ++ColumnIndex)
		{
			FDatabaseColumnInfo& ColumnInfo = ColumnInfos.AddDefaulted_GetRef();
			ColumnInfo.ColumnName = ColumnNames[ColumnIndex];

			ESQLiteColumnType ColumnType = ESQLiteColumnType::Null;
			PreparedStatement.GetColumnTypeByIndex(ColumnIndex, ColumnType);
			switch (ColumnType)
			{
			case ESQLiteColumnType::Integer:
				ColumnInfo.DataType = DBT_INT;
				break;

			case ESQLiteColumnType::Float:
				ColumnInfo.DataType = DBT_FLOAT;
				break;

			case ESQLiteColumnType::String:
				ColumnInfo.DataType = DBT_STRING;
				break;

			case ESQLiteColumnType::Blob:
				ColumnInfo.DataType = DBT_UNKOWN;
				break;

			case ESQLiteColumnType::Null:
			default:
				ColumnInfo.DataType = DBT_UNKOWN;
				break;
			}
		}
		++NumberOfRecords;
	}
	while (PreparedStatement.Step() == ESQLitePreparedStatementStepResult::Row)
	{
		++NumberOfRecords;
	}
	PreparedStatement.Reset();
}

TArray<FDatabaseColumnInfo> FSQLiteResultSet::GetColumnNames() const
{
	return ColumnInfos;
}

int64 FSQLiteResultSet::GetBigInt(const TCHAR* Column) const
{
	int64 Value = 0;
	if (!PreparedStatement.GetColumnValueByName(Column, Value))
	{
		//UE_LOGF(LogDataBase, Log, "SQLITE: Failure retrieving big int value for column [%ls]", Column);
	}
	return Value;
}

float FSQLiteResultSet::GetFloat(const TCHAR* Column) const
{
	float Value = 0.0f;
	if (!PreparedStatement.GetColumnValueByName(Column, Value))
	{
		//UE_LOGF(LogDataBase, Log, "SQLITE: Failure retrieving float value for column [%ls]", Column);
	}
	return Value;
}

int32 FSQLiteResultSet::GetInt(const TCHAR* Column) const
{
	int32 Value = 0;
	if (!PreparedStatement.GetColumnValueByName(Column, Value))
	{
		//UE_LOGF(LogDataBase, Log, "SQLITE: Failure retrieving int value for column [%ls]", Column);
	}
	return Value;
}

FString FSQLiteResultSet::GetString(const TCHAR* Column) const
{
	FString Value;
	if (!PreparedStatement.GetColumnValueByName(Column, Value))
	{
		//UE_LOGF(LogDataBase, Log, "SQLITE: Failure retrieving string value for column [%ls]", Column);
	}
	return Value;
}

int32 FSQLiteResultSet::GetRecordCount() const
{
	return NumberOfRecords;
}

bool FSQLiteResultSet::IsAtEnd() const
{
	return StepResult == ESQLitePreparedStatementStepResult::Done;
}

bool FSQLiteResultSet::HasError() const
{
	return StepResult == ESQLitePreparedStatementStepResult::Error;
}

int32 FSQLiteResultSet::GetErrorCode() const
{
	return StepResultErrorCode;
}

FString FSQLiteResultSet::GetErrorMessage() const
{
	return FString::Printf(TEXT("%s(%d)"), *FSQLiteDatabase::GetErrorForCode(StepResultErrorCode), StepResultErrorCode);
}

void FSQLiteResultSet::MoveToNext()
{
	StepResult = PreparedStatement.Step(&StepResultErrorCode);
}

void FSQLiteResultSet::MoveToFirst()
{
	PreparedStatement.Reset();
	StepResult = PreparedStatement.Step(&StepResultErrorCode);
}
