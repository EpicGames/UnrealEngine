// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SQLiteSupport.h"
#include "SQLiteResultSet.h"
#include "SQLiteDatabase.h"

/**
 * SQLite database file
 */
class SQLITESUPPORT_API FSQLiteDatabaseConnection : public FDataBaseConnection
{
public: 
	/** Closes the database handle and unlocks the file */
	virtual void Close();

	/** Execute a command on the database without storing the result set (if any) */
	virtual bool Execute(const TCHAR* CommandString) override;

	/** Executes the command string on the currently opened database, returning a FSQLiteResultSet in RecordSet. Caller is responsible for freeing RecordSet */
	bool Execute(const TCHAR* CommandString, FSQLiteResultSet*& RecordSet);

	/** Open a SQLite file 
	* @param	ConnectionString	Path to the file that should be opened
	* @param	RemoteConnectionIP	Unused with this implementation
	* @param	ConnectionString	Unused with this implementation
	*/
	virtual bool Open(const TCHAR* ConnectionString, const TCHAR* RemoteConnectionIP, const TCHAR* RemoteConnectionStringOverride);
	
	//Overriding to address warning C4264 - SQLite databases should call Execute(const TCHAR* CommandString, FSQLiteResultSet*& RecordSet) instead
	virtual bool Execute(const TCHAR* CommandString, FDataBaseRecordSet*& RecordSet) override;

	FString GetLastError();

protected:
	FSQLiteDatabase Database;
};
