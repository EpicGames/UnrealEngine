// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Database.h"
#include "SQLitePreparedStatement.h"

/**
 * Result set for SQLite database queries
 */
class SQLITESUPPORT_API FSQLiteResultSet : public FDataBaseRecordSet
{
	//FDatabaseRecordSet implementation
protected:
	virtual void MoveToFirst() override;
	virtual void MoveToNext() override;
	virtual bool IsAtEnd() const override;

public:
	virtual bool HasError() const override;
	virtual int32 GetErrorCode() const override;
	virtual FString GetErrorMessage() const override;
	virtual int32 GetRecordCount() const override;
	virtual FString GetString(const TCHAR* Column) const override;
	virtual int32 GetInt(const TCHAR* Column) const override;
	virtual float GetFloat(const TCHAR* Column) const override;
	virtual int64 GetBigInt(const TCHAR* Column) const override;
	virtual TArray<FDatabaseColumnInfo> GetColumnNames() const override;
	//FDatabaseRecordSet 

	explicit FSQLiteResultSet(FSQLitePreparedStatement&& InPreparedStatement);
	
private:
	FSQLitePreparedStatement PreparedStatement;
	TArray<FDatabaseColumnInfo> ColumnInfos;
	int32 NumberOfRecords = 0;
	int32 StepResultErrorCode = 0;
	ESQLitePreparedStatementStepResult StepResult = ESQLitePreparedStatementStepResult::Done;
};
