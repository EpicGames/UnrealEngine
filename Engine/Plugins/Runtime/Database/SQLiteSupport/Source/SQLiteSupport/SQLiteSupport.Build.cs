// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class SQLiteSupport : ModuleRules
	{
		public SQLiteSupport(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"DatabaseSupport",
					"SQLiteCore",
				}
			);
		}
	}
}
